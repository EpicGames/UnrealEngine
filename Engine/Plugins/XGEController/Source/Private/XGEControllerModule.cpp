// Copyright Epic Games, Inc. All Rights Reserved.
#include "XGEControllerModule.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformNamedPipe.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Modules/ModuleManager.h"
#include "Serialization/MemoryWriter.h"
#include "Modules/ModuleManager.h"
#include "ShaderCompiler.h"

// Comma separated list of executable file names which should be intercepted by XGE.
// Update this list if adding new tasks.
#define XGE_INTERCEPT_EXE_NAMES TEXT("ShaderCompileWorker")

#if PLATFORM_WINDOWS

	#include "Windows/AllowWindowsPlatformTypes.h"
	THIRD_PARTY_INCLUDES_START
	#include <winreg.h>
	THIRD_PARTY_INCLUDES_END
	#include "Windows/HideWindowsPlatformTypes.h"

	#define XGE_CONTROL_WORKER_NAME		TEXT("XGEControlWorker")
#if PLATFORM_WINDOWS_ARM64EC
	#define XGE_CONTROL_WORKER_FILENAME	TEXT("XGEControlWorkerarm64ec.exe")
#elif PLATFORM_CPU_ARM_FAMILY
	#define XGE_CONTROL_WORKER_FILENAME	TEXT("XGEControlWorkerarm64.exe")
#else
	#define XGE_CONTROL_WORKER_FILENAME	TEXT("XGEControlWorker.exe")
#endif

FString GetControlWorkerBinariesPath()
{
	return FPaths::EngineDir() / TEXT("Binaries/Win64/");
}

FString GetControlWorkerExePath()
{
	return FPaths::EngineDir() / TEXT("Binaries/Win64/") / XGE_CONTROL_WORKER_FILENAME;
}

#else
#error XGE Controller is not supported on non-Windows platforms.
#endif

DEFINE_LOG_CATEGORY_STATIC(LogXGEController, Log, Log);

namespace XGEControllerVariables
{
	int32 Enabled = 1;
	FAutoConsoleVariableRef CVarXGEControllerEnabled(
		TEXT("r.XGEController.Enabled"),
		Enabled,
		TEXT("Enables or disables the use of XGE for various build tasks in the engine.\n")
		TEXT("0: Local builds only. \n")
		TEXT("1: Distribute builds using XGE (default)."),
		ECVF_ReadOnly); // Must be set on start-up, e.g. via config ini

	float Timeout = 2.0f;
	FAutoConsoleVariableRef CVarXGEControllerTimeout(
		TEXT("r.XGEController.Timeout"),
		Timeout,
		TEXT("The time, in seconds, to wait after all tasks have been completed before shutting down the controller. (default: 2 seconds)."),
		ECVF_Default);

	int32 AvoidUsingLocalMachine = 1;
	FAutoConsoleVariableRef CVarXGEControllerAvoidUsingLocalMachine(
		TEXT("r.XGEController.AvoidUsingLocalMachine"),
		AvoidUsingLocalMachine,
		TEXT("Whether XGE tasks should avoid running on the local machine (to reduce the oversubscription with local async and out-of-process work).\n")
		TEXT("0: Do not avoid. Distributed tasks will be spawned on all available XGE agents. Can cause oversubscription on the initiator machine. \n")
		TEXT("1: Avoid spawning tasks on the local (initiator) machine except when running a commandlet or -buildmachine is passed (default).\n")
		TEXT("2: Avoid spawning tasks on the local (initiator) machine."),
		FConsoleVariableDelegate::CreateLambda(
			[](IConsoleVariable* /*CVar*/) -> void
			{
				// If XGE has changed its support for local workers, notify shader compiling manager in case local shader compiling thread must be spun up.
				if (GShaderCompilingManager)
				{
					GShaderCompilingManager->OnDistributedShaderCompilingChanged();
				}
			}
		),
		ECVF_Default); // This can be flipped any time XGEControlWorker is restarted

}

namespace XGEController
{
	/** Whether XGE tasks should avoid using the local machine to reduce oversubscription */
	bool AvoidUsingLocalMachine()
	{
		switch (XGEControllerVariables::AvoidUsingLocalMachine)
		{
			case 0:
				return false;
			case 2:
				return true;
			default:
				break;
		}

		static bool bForceUsingLocalMachine = !FPlatformMisc::GetEnvironmentVariable(TEXT("UE-XGEControllerForceUsingLocalMachine")).IsEmpty();
		return !bForceUsingLocalMachine && !GIsBuildMachine && !IsRunningCommandlet();
	}
}

FXGEControllerModule::FXGEControllerModule()
	: bSupported(false)
	, bModuleInitialized(false)
	, bControllerInitialized(false)
	, ControlWorkerDirectory(FPaths::ConvertRelativePathToFull(GetControlWorkerBinariesPath()))
	, RootWorkingDirectory(FString::Printf(TEXT("%sUnrealXGEWorkingDir/"), FPlatformProcess::UserTempDir()))
	, WorkingDirectory(RootWorkingDirectory + FGuid::NewGuid().ToString(EGuidFormats::Digits))
	, PipeName(FString::Printf(TEXT("UnrealEngine-XGE-%s"), *FGuid::NewGuid().ToString(EGuidFormats::Digits)))
	, TasksCS(new FCriticalSection)
	, bShutdown(false)
	, bRestartWorker(false)
	, LastEventTime(0)
{}

FXGEControllerModule::~FXGEControllerModule()
{
	if (TasksCS)
	{
		delete TasksCS;
		TasksCS = nullptr;
	}
}

bool FXGEControllerModule::IsSupported()
{
	if (bControllerInitialized)
	{
		return bSupported;
	}

#if PLATFORM_WINDOWS

	if (!FPlatformProcess::SupportsMultithreading())
	{
		return false; // current implementation requires worker threads
	}

	// Check the command line to see if the XGE controller has been enabled/disabled.
	// This overrides the value of the console variable.
	if (FParse::Param(FCommandLine::Get(), TEXT("xgecontroller")))
	{
		XGEControllerVariables::Enabled = 1;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("noxgecontroller")) ||
		FParse::Param(FCommandLine::Get(), TEXT("noxgeshadercompile")) ||
		FParse::Param(FCommandLine::Get(), TEXT("noshaderworker")))
	{
		XGEControllerVariables::Enabled = 0;
	}

	// Check for a valid installation of Incredibuild by seeing if xgconsole.exe exists.
	if (XGEControllerVariables::Enabled == 1)
	{
		// Try to read from the registry
		FString RegistryPathString;

		if (!FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Xoreax\\IncrediBuild\\Builder"), TEXT("Folder"), RegistryPathString))
		{
			FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\WOW6432Node\\Xoreax\\IncrediBuild\\Builder"), TEXT("Folder"), RegistryPathString);
		}

		if (!RegistryPathString.IsEmpty())
		{
			RegistryPathString = FPaths::Combine(RegistryPathString, TEXT("xgConsole.exe"));
		}

		// Try to find xgConsole.exe from the PATH environment variable
		FString PathString;
		{
			FString EnvString = FPlatformMisc::GetEnvironmentVariable(TEXT("Path"));
			int32 PathStart = EnvString.Find(TEXT("Xoreax\\IncrediBuild"), ESearchCase::IgnoreCase, ESearchDir::FromStart);
			if (PathStart != INDEX_NONE)
			{
				// Move to the front of this string. The +1 is to change the return value from -1 to 0 (signifying our path was at the start of the string) or to move us past the ";"
				PathStart = EnvString.Find(TEXT(";"), ESearchCase::CaseSensitive, ESearchDir::FromEnd, PathStart) + 1;
				int32 PathEnd = EnvString.Find(TEXT(";"), ESearchCase::CaseSensitive, ESearchDir::FromStart, PathStart);
				if (PathEnd == INDEX_NONE)
				{
					PathEnd = EnvString.Len();
				}

				FString Directory = EnvString.Mid(PathStart, PathEnd - PathStart);
				PathString = FPaths::Combine(Directory, TEXT("xgConsole.exe"));
			}
		}

		// List of possible paths to xgconsole.exe
		const FString Paths[] =
		{
			*RegistryPathString,
			TEXT("C:\\Program Files\\Xoreax\\IncrediBuild\\xgConsole.exe"),
			TEXT("C:\\Program Files (x86)\\Xoreax\\IncrediBuild\\xgConsole.exe"),
			*PathString
		};

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		bool bFound = false;
		for (const FString& Path : Paths)
		{
			if (!Path.IsEmpty() && PlatformFile.FileExists(*Path))
			{
				bFound = true;
				XGConsolePath = Path;
				break;
			}
		}

		if (!bFound)
		{
			UE_LOGF(LogXGEController, Log, "Cannot use XGE Controller as Incredibuild is not installed on this machine.");
			XGEControllerVariables::Enabled = 0;
		}
		else
		{
			// xgConsole.exe has been found.
			// Check we have a compatible version of XGE by finding the version registry key.
			int32 Version = 0;

			HKEY RegistryKey;
			if (RegOpenKeyEx(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Xoreax\\IncrediBuild\\Builder"),              0, KEY_READ, &RegistryKey) == ERROR_SUCCESS ||
				RegOpenKeyEx(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\WOW6432Node\\Xoreax\\IncrediBuild\\Builder"), 0, KEY_READ, &RegistryKey) == ERROR_SUCCESS)
			{
				DWORD Type;
				BYTE Buffer[512] = {};
				DWORD Size = sizeof(Buffer);
				if (RegQueryValueEx(RegistryKey, TEXT("Version"), nullptr, &Type, Buffer, &Size) == ERROR_SUCCESS)
				{
					if (Type == REG_SZ && Size > 0 && FCString::IsNumeric((TCHAR*)Buffer))
					{
						Version = FCString::Atoi((TCHAR*)Buffer);
					}
				}

				RegCloseKey(RegistryKey);
			}

			if (Version == 0)
			{
				UE_LOGF(LogXGEController, Display, "Cannot determine XGE version. XGE Shader compilation with the interception interface may fail [%ls].", *XGConsolePath);
			}
			else if (Version < 1002867)
			{
				UE_LOGF(LogXGEController, Display, "XGE version 8.01 (build 1867) or higher is required for XGE shader compilation with the interception interface [%ls].", *XGConsolePath);
				XGEControllerVariables::Enabled = 0;
			}

			// check if build service is running - without this, the build will make no progress
			const TCHAR* XGEBuildServiceExecutableName = TEXT("BuildService.exe");
			if (!FPlatformProcess::IsApplicationRunning(XGEBuildServiceExecutableName))
			{
				UE_LOGF(LogXGEController, Display, "XGE's background service (%ls) is not running - service is likely disabled on this machine [%ls].", XGEBuildServiceExecutableName, *XGConsolePath);
				XGEControllerVariables::Enabled = 0;
			}

			if (!PlatformFile.FileExists(*GetControlWorkerExePath()))
			{
				UE_LOGF(LogXGEController, Display, "XGEControlWorker.exe does not exist, XGE may be disabled in your Build Configuration, cannot use XGE [%ls].", *XGConsolePath);
				XGEControllerVariables::Enabled = 0;
			}
		}
	}

	return (bSupported = (XGEControllerVariables::Enabled == 1));

#else

	// Not supported on other platforms.
	return false;

#endif
}

void FXGEControllerModule::CleanWorkingDirectory()
{
	// Only clean the directory if we are the only instance running,
	// and we're not running in multi-process mode.
	if (FPlatformProcess::IsFirstInstance() && !FParse::Param(FCommandLine::Get(), TEXT("Multiprocess")))
	{
		UE_LOGF(LogXGEController, Log, "Cleaning working directory: %ls", *RootWorkingDirectory);
		IFileManager::Get().DeleteDirectory(*RootWorkingDirectory, false, true);
	}
}

void FXGEControllerModule::StartupModule()
{
	check(!bModuleInitialized);

	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureType(), this);
	
	bModuleInitialized = true;
}

void FXGEControllerModule::ShutdownModule()
{
	check(bModuleInitialized);

	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureType(), this);

	if (IsSupported())
	{
		bShutdown = true;
		
		if (bControllerInitialized)
		{
			WriteOutThreadEvent->Trigger();

			// Wait for worker threads to exit
			if (WriteOutThreadFuture.IsValid())
			{
				WriteOutThreadFuture.Wait();
				WriteOutThreadFuture = TFuture<void>();
			}
		}

		// Cancel any remaining tasks
		for (auto Iter = DispatchedTasks.CreateIterator(); Iter; ++Iter)
		{
			FDistributedBuildTask* Task = Iter.Value();
			Task->Cancel();
			delete Task;
		}

		FDistributedBuildTask* Task;
		while (PendingTasks.Dequeue(Task))
		{
			Task->Cancel();
			delete Task;
		}

		PendingTasks.Empty();
		DispatchedTasks.Empty();
	}

	CleanWorkingDirectory();
	bModuleInitialized = false;
	bControllerInitialized = false;
}

void FXGEControllerModule::InitializeController()
{
	if (ensureAlwaysMsgf(!bControllerInitialized, TEXT("Multiple initialization of the XGE controller!")))
	{
		CleanWorkingDirectory();
		bShutdown = false;
		// The actual initialization happens in IsSupported() when it is called with bControllerInitialized being false
		if (IsSupported())
		{
			WriteOutThreadFuture = Async(EAsyncExecution::Thread, [this]() { WriteOutThreadProc(); });
			bControllerInitialized = true;

			UE_LOGF(LogXGEController, Display, "Initialized XGE controller. XGE tasks will %lsbe spawned on this machine.",
				XGEController::AvoidUsingLocalMachine() ? TEXT("not ") : TEXT("")
				);

		}
	}
}

bool FXGEControllerModule::SupportsLocalWorkers()
{
	// XGE supports local workers if it's not explicitly disabled
	const bool bAvoidLocalWorkers = XGEController::AvoidUsingLocalMachine();
	return !bAvoidLocalWorkers;
}

void FXGEControllerModule::WriteOutThreadProc()
{
	do
	{
		bRestartWorker = false;

		while (!AreTasksPending() && !bShutdown)
		{
			UE_LOGF(LogXGEController, Verbose, "Waiting for new tasks.");
			WriteOutThreadEvent->Wait();
		}

		if (bShutdown)
		{
			UE_LOGF(LogXGEController, Verbose, "Shutting down communication with the XGE controller.");
			return;
		}

		// To handle spaces in the engine path, we just pass the XGEController.exe filename to xgConsole,
		// and set the working directory of xgConsole.exe to the engine binaries folder below.
		FString XGConsoleArgs = FString::Printf(TEXT("/VIRTUALIZEDIRECTX /allowremote=\"%s\" %s /allowintercept=\"%s\" /title=\"Unreal Engine XGE Tasks\" /monitordirs=\"%s\" /command=\"%s -xgecontroller %s\""),
			XGE_INTERCEPT_EXE_NAMES,
			(XGEController::AvoidUsingLocalMachine() && (IsRunningCommandlet() || !GShaderCompilingManager->IgnoreAllThrottling())) ? TEXT("/avoidlocal=ON") : TEXT(""),
			XGE_CONTROL_WORKER_NAME,
			*WorkingDirectory,
			XGE_CONTROL_WORKER_FILENAME,
			*PipeName);

		// Create the output pipe as a server...
		if (!OutputNamedPipe.Create(FString::Printf(TEXT("\\\\.\\pipe\\%s-A"), *PipeName), true, false))
		{
			UE_LOGF(LogXGEController, Fatal, "Failed to create the output XGE named pipe.");
		}

		const int32 PriorityModifier = 0;	// normal by default. Interactive use case shouldn't be affected as the jobs will avoid local machine
		// Start the controller process
		uint32 XGConsoleProcID = 0;
		UE_LOGF(LogXGEController, Verbose, "Launching xgConsole");
		BuildProcessHandle = FPlatformProcess::CreateProc(*XGConsolePath, *XGConsoleArgs, false, false, true, &XGConsoleProcID, PriorityModifier, *ControlWorkerDirectory, nullptr);
		if (!BuildProcessHandle.IsValid())
		{
			UE_LOGF(LogXGEController, Fatal, "Failed to launch the XGE control worker process.");
		}

		// If the engine crashes, we don't get a chance to kill the build process.
		// Start up the build monitor process to monitor for engine crashes.
		uint32 BuildMonitorProcessID;
		UE_LOGF(LogXGEController, Verbose, "Launching XGEController to fan out tasks");
		FString XGMonitorArgs = FString::Printf(TEXT("-xgemonitor %d %d"), FPlatformProcess::GetCurrentProcessId(), XGConsoleProcID);
		FProcHandle BuildMonitorHandle = FPlatformProcess::CreateProc(*GetControlWorkerExePath(), *XGMonitorArgs, true, false, false, &BuildMonitorProcessID, PriorityModifier, nullptr, nullptr);
		FPlatformProcess::CloseProc(BuildMonitorHandle);

		// Wait for the controller to connect to the output pipe
		if (!OutputNamedPipe.OpenConnection())
		{
			UE_LOGF(LogXGEController, Fatal, "Failed to open a connection on the output XGE named pipe.");
		}

		// Connect the input pipe (controller is the server)...
		if (!InputNamedPipe.Create(FString::Printf(TEXT("\\\\.\\pipe\\%s-B"), *PipeName), false, false))
		{
			UE_LOGF(LogXGEController, Fatal, "Failed to connect the input XGE named pipe.");
		}

		// Pass the xgConsole process ID to the XGE control worker, so it can terminate the build on exit
		if (!OutputNamedPipe.WriteBytes(sizeof(XGConsoleProcID), &XGConsoleProcID))
		{
			UE_LOGF(LogXGEController, Fatal, "Failed to pass xgConsole process ID to XGE control worker.");
		}

		LastEventTime = FPlatformTime::Cycles();

		// Launch the output thread
		ReadBackThreadFuture = Async(EAsyncExecution::Thread, [this]() { ReadBackThreadProc(); });

		// Main Tasks Loop
		TArray<uint8> WriteBuffer;
		while (true)
		{
			WriteBuffer.Reset();

			// Wait for new tasks to arrive, with a timeout...
			while (!bShutdown && !bRestartWorker && !AreTasksPending())
			{
				uint32 LastTime = (uint32)FPlatformAtomics::InterlockedAdd(reinterpret_cast<volatile int32*>(&LastEventTime), 0);
				float ElapsedSeconds = (FPlatformTime::Cycles() - LastTime) * FPlatformTime::GetSecondsPerCycle();
				float SecondsToWait = XGEControllerVariables::Timeout - ElapsedSeconds;

				if (!WriteOutThreadEvent->Wait(FMath::CeilToInt(SecondsToWait * 1000.0f)) && !AreTasksDispatchedOrPending())
				{
					// Timed out, and no more pending or dispatched tasks. End the current build.
					bRestartWorker = true;
					break;
				}
			}

			if (bShutdown || bRestartWorker)
				break;

			// Take one task from the pending queue.
			FDistributedBuildTask* Task = nullptr;
			{
				FScopeLock Lock(TasksCS);
				PendingTasks.Dequeue(Task);
			}

			if (Task)
			{
				WriteBuffer.Reset();
				WriteBuffer.AddUninitialized(sizeof(uint32));

				FMemoryWriter Writer(WriteBuffer, false, true);

				Writer << Task->ID;
				Writer << Task->CommandData.Command;

				// XGE only prepends its own sentinel flag. Workers that don't care about -xge_int should ignore it.
				FString WorkerParameters = FString::Printf(TEXT("-xge_int %s"), *Task->CommandData.ExtraCommandArgs);

				Writer << WorkerParameters;
				*reinterpret_cast<uint32*>(WriteBuffer.GetData()) = WriteBuffer.Num() - sizeof(uint32);

				// Move the tasks to the dispatched tasks map before launching it
				{
					FScopeLock Lock(TasksCS);
					DispatchedTasks.Add(Task->ID, Task);
				}

				if (!OutputNamedPipe.WriteBytes(WriteBuffer.Num(), WriteBuffer.GetData()))
				{
					// Error occurred whilst writing task args to the named pipe.
					// It's likely the controller process was terminated.
					bRestartWorker = true;
				}

				// Update the last event time.
				FPlatformAtomics::InterlockedExchange(reinterpret_cast<volatile int32*>(&LastEventTime), FPlatformTime::Cycles());
			}
		}

		// Destroy the output named pipe. This signals the worker to exit, if it hasn't already.
		OutputNamedPipe.Destroy();

		// Wait for the read back thread to exit. This will happen when the input pipe is closed by the worker.
		if (ReadBackThreadFuture.IsValid())
		{
			ReadBackThreadFuture.Wait();
			ReadBackThreadFuture = TFuture<void>();
		}

		// Wait for the build process
		if (BuildProcessHandle.IsValid())
		{
			if (FPlatformProcess::IsProcRunning(BuildProcessHandle))
				FPlatformProcess::WaitForProc(BuildProcessHandle);

			FPlatformProcess::CloseProc(BuildProcessHandle);
		}

		// Reclaim dispatched (incomplete) tasks 
		for (auto Iter = DispatchedTasks.CreateIterator(); Iter; ++Iter)
			PendingTasks.Enqueue(Iter.Value());

		DispatchedTasks.Reset();

	} while (!bShutdown);
}

void FXGEControllerModule::ReadBackThreadProc()
{
	while (!bShutdown && !bRestartWorker)
	{
		FTaskResponse CompletedTaskResponse;
		if (!InputNamedPipe.ReadBytes(sizeof(CompletedTaskResponse), &CompletedTaskResponse))
		{
			// The named pipe was closed or had an error.
			// Instruct the write-out thread to restart the worker, then exit.
			bRestartWorker = true;
		}
		else
		{
			// Update the last event time.
			FPlatformAtomics::InterlockedExchange(reinterpret_cast<volatile int32*>(&LastEventTime), FPlatformTime::Cycles());

			// We've read a completed task response from the controller.
			// Find the task in the map and complete the promise.
			FDistributedBuildTask* Task;
			{
				FScopeLock Lock(TasksCS);
				Task = DispatchedTasks.FindAndRemoveChecked(CompletedTaskResponse.ID);
			}

			Task->Finalize(CompletedTaskResponse.ReturnCode);
			delete Task;
		}

		WriteOutThreadEvent->Trigger();
	}

	InputNamedPipe.Destroy();
}

FString FXGEControllerModule::CreateUniqueFilePath()
{
	check(bSupported);
	return FString::Printf(TEXT("%s/%d-xge"), *WorkingDirectory, NextFileID.Increment());
}

TFuture<FDistributedBuildTaskResult> FXGEControllerModule::EnqueueTask(const FTaskCommandData& CommandData)
{
	check(bSupported);

	TPromise<FDistributedBuildTaskResult> Promise;
	TFuture<FDistributedBuildTaskResult> Future = Promise.GetFuture();

	// Enqueue the new task
	FDistributedBuildTask* Task = new FDistributedBuildTask(NextTaskID.Increment(), CommandData, MoveTemp(Promise));
	{
		FScopeLock Lock(TasksCS);
		PendingTasks.Enqueue(Task);
	}

	WriteOutThreadEvent->Trigger();

	return MoveTemp(Future);
}

XGECONTROLLER_API FXGEControllerModule& FXGEControllerModule::Get()
{
	return FModuleManager::LoadModuleChecked<FXGEControllerModule>(TEXT("XGEController"));
}

IMPLEMENT_MODULE(FXGEControllerModule, XGEController);
