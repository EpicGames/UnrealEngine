// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Queue.h"
#include "DistributedBuildControllerInterface.h"
#include "Windows/WindowsPlatformNamedPipe.h"

class FXGEControllerModule : public IDistributedBuildController
{
	/** Whether this system can use XGE shader compilation. */
	bool bSupported;
	/** Whether the module was initialized. */
	bool bModuleInitialized;
	/** Whether the controller was successfully initialized. This is only possible when bSupported is true. */
	bool bControllerInitialized;

	FThreadSafeCounter NextFileID;
	FThreadSafeCounter NextTaskID;

	FProcHandle BuildProcessHandle;

	const FString ControlWorkerDirectory;
	const FString RootWorkingDirectory;
	const FString WorkingDirectory;
	const FString PipeName;
	FString XGConsolePath;

	// Taken when accessing the PendingTasks and DispatchedTasks members.
	FCriticalSection* TasksCS;

	// Queue of tasks submitted by the engine, but not yet dispatched to the controller.
	TQueue<FDistributedBuildTask*> PendingTasks;

	// Map of tasks dispatched to the controller and running within XGE, that have not yet finished.
	TMap<uint32, FDistributedBuildTask*> DispatchedTasks;

	bool bShutdown;
	bool bRestartWorker;
	TFuture<void> WriteOutThreadFuture;
	TFuture<void> ReadBackThreadFuture;

	/** event for the thread that communicates with the XGE controller */
	FEventRef WriteOutThreadEvent;

	// We need two pipes, as the named pipe API does not support simultaneous read/write on two threads.
	FPlatformNamedPipe InputNamedPipe, OutputNamedPipe;

	volatile uint32 LastEventTime;

	inline bool AreTasksPending()
	{
		FScopeLock Lock(TasksCS);
		return !PendingTasks.IsEmpty();
	}

	inline bool AreTasksDispatchedOrPending()
	{
		FScopeLock Lock(TasksCS);
		return DispatchedTasks.Num() > 0 || !PendingTasks.IsEmpty();
	}

public:
	FXGEControllerModule();
	virtual ~FXGEControllerModule();

	virtual void StartupModule() override final;
	virtual void ShutdownModule() override final;

	virtual void InitializeController() override final;
	virtual bool SupportsLocalWorkers() override final;

	virtual const FString GetName() override final { return FString("XGE Controller"); };

	XGECONTROLLER_API virtual bool IsSupported() override final;

	virtual FString CreateUniqueFilePath() override final;
	virtual TFuture<FDistributedBuildTaskResult> EnqueueTask(const FTaskCommandData& CommandData) override final;
	
	XGECONTROLLER_API static FXGEControllerModule& Get();

	void WriteOutThreadProc();
	void ReadBackThreadProc();

	void CleanWorkingDirectory();
};
