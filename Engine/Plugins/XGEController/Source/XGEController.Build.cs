// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class XGEController : ModuleRules
{
	public XGEController(ReadOnlyTargetRules TargetRules)
		: base(TargetRules)
	{
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",

			// all these below are only to get access to GShaderCompilingManager
			"Slate",
			"RHI",
			"RenderCore",
			"Engine",
		});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DistributedBuildInterface",
			}
		);
	}
}
