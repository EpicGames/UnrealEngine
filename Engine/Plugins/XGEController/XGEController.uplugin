{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "XGE Controller",
	"Description" : "Adds support for shader compiling distribution using XGE",
	"Category" : "Build Distribution",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,

	"Modules" :
	[
		{
			"Name" : "XGEController",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "EarliestPossible",
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}